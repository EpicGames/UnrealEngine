// Copyright Epic Games, Inc. All Rights Reserved.

#include "Delegates/Delegate.h"
#include "Framework/Docking/TabManager.h"
#include "IUndoHistoryEditorModule.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SUndoHistory.h"

const FName UndoHistoryTabName("UndoHistory");

class FUndoHistoryEditorModule : public IUndoHistoryEditorModule
{
public:

	virtual void StartupModule() override
	{
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(UndoHistoryTabName, FOnSpawnTab::CreateRaw(this, &FUndoHistoryEditorModule::HandleSpawnSettingsTab))
			.SetDisplayName(NSLOCTEXT("FUndoHistoryModule", "UndoHistoryTabTitle", "Undo History"))
			.SetTooltipText(NSLOCTEXT("FUndoHistoryModule", "UndoHistoryTooltipText", "Open the Undo History tab."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "UndoHistory.TabIcon"))
			.SetAutoGenerateMenuEntry(false);
	}
	
	virtual void ShutdownModule() override
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(UndoHistoryTabName);
	}
	
	virtual void ExecuteOpenUndoHistory() override
	{
		FGlobalTabmanager::Get()->TryInvokeTab(UndoHistoryTabName);
	}

private:
	
	/** Handles creating the project settings tab. */
	TSharedRef<SDockTab> HandleSpawnSettingsTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
		[
			SNew(SUndoHistory)
		];
		return DockTab;
	}
};

IMPLEMENT_MODULE(FUndoHistoryEditorModule, UndoHistoryEditor);

