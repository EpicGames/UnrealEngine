// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UndoHistorySettings.generated.h"


/**
 * Implements the settings for the UndoHistory.
 */
UCLASS(config = EditorPerProjectUserSettings)
class UUndoHistorySettings
	: public UObject
{
public:
	GENERATED_BODY()

	/** True when the UndoHistory is showing transaction details. */
	UPROPERTY(config)
	bool bShowTransactionDetails;
};
