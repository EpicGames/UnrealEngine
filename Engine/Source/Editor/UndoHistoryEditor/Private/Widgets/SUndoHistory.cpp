// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SUndoHistory.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SUndoHistoryTableRow.h"
#include "Widgets/Input/SComboButton.h"
#include "Editor/TransBuffer.h"
#include "UndoHistorySettings.h"
#include "Misc/ScopedSlowTask.h"
#include "Widgets/SUndoHistoryDetails.h"

#define LOCTEXT_NAMESPACE "SUndoHistory"


// Helper function for TransactionInfo
const FTransaction* SUndoHistory::FTransactionInfo::GetTransaction() const
{
	if (GEditor && GEditor->Trans)
	{
		return GEditor->Trans->GetTransaction(QueueIndex);
	}
	return nullptr;
}

/* SUndoHistory interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SUndoHistory::Construct( const FArguments& InArgs )
{
	LastActiveTransactionIndex = INDEX_NONE;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(Splitter, SSplitter)
					.Orientation(EOrientation::Orient_Vertical)

					+ SSplitter::Slot()
						[
							SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.Padding(FMargin(2.0f))
								.AutoHeight()
								[
									SNew(SHorizontalBox)

									+SHorizontalBox::Slot()
										.AutoWidth()
										.Padding( 0.0f, 0.0f, 4.0f, 0.0f )
										[
											SNew( SImage )
												.Image( FAppStyle::GetBrush( "LevelEditor.Tabs.Details" ) )
										]

									+SHorizontalBox::Slot()
										.HAlign(HAlign_Left)
										[
											SNew(STextBlock)
												.Text(LOCTEXT("TransactionHistoryLabel", "Transactions"))
										]
								]
							
								+ SVerticalBox::Slot()
								[
									SNew(SBorder)
										.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
										.Padding(FMargin(4.0f, 1.0f))
										[
											SAssignNew(UndoListView, SListView<TSharedPtr<FTransactionInfo> >)
												.ListItemsSource(&UndoList)
												.SelectionMode(ESelectionMode::Single)
												.OnGenerateRow(this, &SUndoHistory::HandleUndoListGenerateRow)
												.OnMouseButtonDoubleClick(this, &SUndoHistory::HandleUndoListJumpToTransaction)
												.OnSelectionChanged(this, &SUndoHistory::HandleUndoListSelectionChanged)
												.HeaderRow
												(
													SNew(SHeaderRow)
														.Visibility(EVisibility::Collapsed)

													+ SHeaderRow::Column("Title")
														.FillWidth(80.0f)

													+ SHeaderRow::Column("UndoBarrierButton")
														.FixedWidth(50.0f)

													+ SHeaderRow::Column("JumpToButton")
														.FixedWidth(50.0f)
												)
										]
								]

						]

					+ SSplitter::Slot()
						.Value(0.4f)
						[
							SNew(SVerticalBox)
							.Visibility(this, &SUndoHistory::HandleUndoHistoryDetailsVisibility)
							+SVerticalBox::Slot()
							.Padding(FMargin(2.0f))
							.AutoHeight()
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.AutoWidth()
									.Padding( 0.0f, 0.0f, 4.0f, 0.0f )
									[
										SNew(SImage)
											.Image(FAppStyle::GetBrush( "LevelEditor.Tabs.Details" ))
									]

								+ SHorizontalBox::Slot()
									.Padding(FMargin(4.0f, 0.0f))
									[
										SNew(STextBlock)
											.TextStyle(FAppStyle::Get(), "NormalText")
											.Text(LOCTEXT("TransactionDetailsLabel", "Transaction Details"))
									]
							]
							
							+SVerticalBox::Slot()
							[
								SAssignNew(UndoDetailsView, SUndoHistoryDetails)
							]
						]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
							.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
							.ToolTipText(LOCTEXT("UndoBufferFullToolTip", "The undo buffer has reached its maximum capacity, transactions will be deleted from the top."))
							.Text(FText::FromString(FString(TEXT("\xf071 "))) /*fa-exclamation-triangle*/)
							.ColorAndOpacity(FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton.Danger").Normal.TintColor)
							.Visibility(this, &SUndoHistory::HandleUndoWarningVisibility)
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						// Undo size
						SNew(STextBlock)
							.Text(this, &SUndoHistory::HandleUndoSizeTextBlockText)
					]

				+ SHorizontalBox::Slot() 
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					.HAlign(HAlign_Left)
					[
						// Discard history button
						SAssignNew(DiscardButton, SButton)
							.ForegroundColor(FSlateColor::UseForeground())
							.ButtonStyle(FAppStyle::Get(), "ToggleButton")
 							.OnClicked(this, &SUndoHistory::HandleDiscardHistoryButtonClicked)
							.ToolTipText(LOCTEXT("DiscardHistoryButtonToolTip", "Discard the Undo History."))
							[
								SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
							]
					]

				+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					[
						SNew(SComboButton)
							.ContentPadding(0.f)
							.ForegroundColor(FSlateColor::UseForeground())
							.ButtonStyle(FAppStyle::Get(), "ToggleButton")
							.OnGetMenuContent(this, &SUndoHistory::GetViewButtonContent)
							.ButtonContent()
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									[
										SNew(SImage).Image(FAppStyle::GetBrush("GenericViewButton"))
									]
							]
					]
			]
	];

	ReloadUndoList();

	if (GEditor != nullptr && GEditor->Trans != nullptr)
	{
		UTransBuffer* TransBuffer = CastChecked<UTransBuffer>(GEditor->Trans);
		TransBuffer->OnUndoBufferChanged().AddRaw(this, &SUndoHistory::OnUndoBufferChanged);
		TransBuffer->OnTransactionStateChanged().AddRaw(this, &SUndoHistory::OnTransactionStateChanged);
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SUndoHistory::~SUndoHistory()
{
	if (GEditor != nullptr && GEditor->Trans != nullptr)
	{
		UTransBuffer* TransBuffer = CastChecked<UTransBuffer>(GEditor->Trans);
		TransBuffer->OnUndoBufferChanged().RemoveAll(this);
		TransBuffer->OnTransactionStateChanged().RemoveAll(this);
	}
}

void SUndoHistory::OnUndoBufferChanged()
{
	if (!IsEngineExitRequested())
	{
		ReloadUndoList();
		SelectLastTransaction();
	}
}


void SUndoHistory::OnTransactionStateChanged(const FTransactionContext& TransactionContext, ETransactionStateEventType TransactionState)
{
	if (TransactionState == ETransactionStateEventType::TransactionFinalized)
	{
		SelectLastTransaction();
	}
}

void SUndoHistory::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// reload the transaction list if necessary
	if (GEditor == nullptr || GEditor->Trans == nullptr)
	{
		ReloadUndoList();
	}
	else if (GEditor != nullptr && GEditor->Trans != nullptr)
	{
		// update the selected transaction if necessary
		int32 ActiveTransactionIndex = GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount() - 1;

		if (UndoList.IsValidIndex(ActiveTransactionIndex) && (ActiveTransactionIndex != LastActiveTransactionIndex))
		{
			UndoListView->SetSelection(UndoList[ActiveTransactionIndex]);
			LastActiveTransactionIndex = ActiveTransactionIndex;
		}
	}
}


/* SUndoHistory implementation
 *****************************************************************************/

void SUndoHistory::ReloadUndoList()
{
	UndoList.Empty();

	if ((GEditor == nullptr) || (GEditor->Trans == nullptr))
	{
		return;
	}

	for (int32 QueueIndex = 0; QueueIndex < GEditor->Trans->GetQueueLength(); ++QueueIndex)
	{
		UndoList.Add(MakeShared<FTransactionInfo>(QueueIndex));
	}

	UndoListView->RequestListRefresh();
}

void SUndoHistory::SelectLastTransaction()
{
	if (GEditor != nullptr && GEditor->Trans != nullptr)
	{
		LastActiveTransactionIndex = GEditor->Trans->GetQueueLength() - 1;
		if (UndoList.IsValidIndex(LastActiveTransactionIndex))
		{
			const TSharedPtr<FTransactionInfo>& TransactionInfo = UndoList[LastActiveTransactionIndex];
			const FTransaction* InnerTransaction = TransactionInfo.IsValid() ? TransactionInfo->GetTransaction() : nullptr;
			if (InnerTransaction)
			{
				UndoListView->SetSelection(TransactionInfo);
				UndoDetailsView->SetSelectedTransaction(InnerTransaction->GetHistoricDiff());
			}
		}
		else
		{
			UndoDetailsView->Reset();
		}
	}
}


/* SUndoHistory callbacks
 *****************************************************************************/

FReply SUndoHistory::HandleDiscardHistoryButtonClicked()
{
	if ((GEditor != nullptr) && (GEditor->Trans != nullptr))
	{
		GEditor->Trans->Reset(LOCTEXT("DiscardHistoryReason", "Discard undo history."));
		ReloadUndoList();
		UndoDetailsView->Reset();
	}

	return FReply::Handled();
}


TSharedRef<ITableRow> SUndoHistory::HandleUndoListGenerateRow(TSharedPtr<FTransactionInfo> TransactionInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SUndoHistoryTableRow, OwnerTable)
		.OnGotoTransactionClicked(this, &SUndoHistory::HandleGoToTransaction)
		.OnUndoBarrierButtonClicked(this, &SUndoHistory::HandleUndoBarrierButtonClicked)
		.IsApplied(this, &SUndoHistory::HandleUndoListRowIsApplied, TransactionInfo->QueueIndex)
		.IsCurrentUndoBarrier(this, &SUndoHistory::HandleIsCurrentUndoBarrier, TransactionInfo->QueueIndex)
		.IsLastTransactionIndex(this, &SUndoHistory::HandleIsLastTransactionIndex, TransactionInfo->QueueIndex)
		.QueueIndex(TransactionInfo->QueueIndex)
		.Transaction(TransactionInfo->GetTransaction());
}

void SUndoHistory::HandleGoToTransaction(const FGuid& TargetTransactionId)
{
	TSharedPtr<FTransactionInfo>* Transaction = UndoList.FindByPredicate([TargetTransactionId](TSharedPtr<FTransactionInfo> TransactionInfo)
	{
		if (const FTransaction* InnerTransaction = TransactionInfo->GetTransaction())
		{
			return InnerTransaction->GetId() == TargetTransactionId;
		}
		return false;
	});

	if (Transaction != nullptr)
	{
		HandleUndoListJumpToTransaction(*Transaction);
	}
}

bool SUndoHistory::HandleIsCurrentUndoBarrier(int32 InQueueIndex) const
{
	if ((GEditor == nullptr) || (GEditor->Trans == nullptr))
	{
		return false;
	}

	return InQueueIndex == GEditor->Trans->GetCurrentUndoBarrier() - 1;
}

bool SUndoHistory::HandleIsLastTransactionIndex(int32 InQueueIndex) const
{
	if ((GEditor == nullptr) || (GEditor->Trans == nullptr))
	{
		return false;
	}

	return InQueueIndex == (GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount() - 1);
}

void SUndoHistory::HandleUndoBarrierButtonClicked(int32 InQueueIndex)
{
	if (HandleIsCurrentUndoBarrier(InQueueIndex))
	{
		GEditor->Trans->RemoveUndoBarrier();
	}
	else if (HandleIsLastTransactionIndex(InQueueIndex))
	{
		GEditor->Trans->SetUndoBarrier();
	}
}

bool SUndoHistory::HandleUndoListRowIsApplied(int32 QueueIndex) const
{
	if ((GEditor == nullptr) || (GEditor->Trans == nullptr))
	{
		return false;
	}

	return (QueueIndex < (GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount()));
}


void SUndoHistory::HandleUndoListSelectionChanged(TSharedPtr<FTransactionInfo> InItem, ESelectInfo::Type SelectInfo)
{
	if (!InItem.IsValid() || (GEditor == nullptr) || (GEditor->Trans == nullptr))
	{
		return;
	}

	if (const FTransaction* InnerTransaction = InItem->GetTransaction())
	{
		UndoDetailsView->SetSelectedTransaction(InnerTransaction->GetHistoricDiff());
	}

	if (SelectInfo == ESelectInfo::OnMouseClick || SelectInfo == ESelectInfo::Direct)
	{
		// Use Private_SetItemSelection to avoid creating an OnSelectionChanged broadcast
		UndoListView->Private_SetItemSelection(InItem, true, false);
		UndoListView->RequestScrollIntoView(InItem);
	}


}

void SUndoHistory::HandleUndoListJumpToTransaction(TSharedPtr<FTransactionInfo> InItem)
{
	if (!InItem.IsValid() || (GEditor == nullptr) || (GEditor->Trans == nullptr))
	{
		return;
	}

	LastActiveTransactionIndex = GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount() - 1;

	const int32 RemainingUndoRedo = FMath::Abs(InItem->QueueIndex - LastActiveTransactionIndex);

 	FScopedSlowTask SlowTask(static_cast<float>(RemainingUndoRedo), LOCTEXT("ReplayingTransactions", "Replaying Transactions..."));
 	SlowTask.MakeDialogDelayed(0.5f);

	const auto ProgressLambda = [&SlowTask](const int32 InTransactionIndex)
	{
		const FTransaction* Transaction = GEditor->Trans->GetTransaction(InTransactionIndex);

		if (Transaction)
		{
			SlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("ReplayingTransactionFmt", "Replaying {0}"), Transaction->GetTitle()));
		}
	};

	while (InItem->QueueIndex > LastActiveTransactionIndex)
	{
		if (!GEditor->Trans->Redo())
		{	
			break;
		}

		ProgressLambda(LastActiveTransactionIndex);
		++LastActiveTransactionIndex;
	}

	while (InItem->QueueIndex < LastActiveTransactionIndex)
	{
		if (!GEditor->Trans->Undo())
		{
			break;
		}

		ProgressLambda(LastActiveTransactionIndex);
		--LastActiveTransactionIndex;
	}

	UndoListView->RequestScrollIntoView(InItem);
	UndoListView->SetSelection(InItem);
}


EVisibility SUndoHistory::HandleUndoHistoryDetailsVisibility() const
{
	return IsShowingTransactionDetails() ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SUndoHistory::HandleUndoSizeTextBlockText() const
{
	if ((GEditor == nullptr) || (GEditor->Trans == nullptr))
	{
		return FText::GetEmpty();
	}

	return FText::Format(LOCTEXT("TransactionCountF", "{0} {0}|plural(one=Transaction,other=Transactions) ({1})"), FText::AsNumber(UndoList.Num()), FText::AsMemory(GEditor->Trans->GetUndoSize()));
}

EVisibility SUndoHistory::HandleUndoWarningVisibility() const
{
	if ((GEditor == nullptr) || (GEditor->Trans == nullptr))
	{
		return EVisibility::Collapsed;
	}

	UTransBuffer* TransBuffer = CastChecked<UTransBuffer>(GEditor->Trans);

	return TransBuffer->GetUndoSize() > TransBuffer->MaxMemory ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SUndoHistory::GetViewButtonContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	MenuBuilder.BeginSection("AssetThumbnails", LOCTEXT("ShowHeading", "Show"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleShowTransactionDetails", "Show transactions details."),
			LOCTEXT("ToggleShowTransactionDetailsToolTip", "When enabled, display additional information about transactions."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(const_cast<SUndoHistory*>(this), &SUndoHistory::ToggleShowTransactionDetails),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SUndoHistory::IsShowingTransactionDetails)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

/** FILTERS */
void SUndoHistory::ToggleShowTransactionDetails()
{
	const bool bEnableFlag = !IsShowingTransactionDetails();

	UUndoHistorySettings* Settings = GetMutableDefault<UUndoHistorySettings>();
	Settings->bShowTransactionDetails = bEnableFlag;
}

bool SUndoHistory::IsShowingTransactionDetails() const
{
	return GetDefault<UUndoHistorySettings>()->bShowTransactionDetails;
}

#undef LOCTEXT_NAMESPACE