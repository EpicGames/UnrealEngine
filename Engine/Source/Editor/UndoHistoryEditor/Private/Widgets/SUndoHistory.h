// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Misc/ITransaction.h"

class FTransaction;

/**
 * Implements the undo history panel.
 */
class SUndoHistory
	: public SCompoundWidget
{
	// Structure for transaction information.
	struct FTransactionInfo
	{
		// Holds the transactions index in the transaction queue.
		int32 QueueIndex;

		// Creates and initializes a new instance.
		FTransactionInfo( int32 InQueueIndex )
			: QueueIndex(InQueueIndex)			
		{ }

		const FTransaction* GetTransaction() const;
	};

public:

	SLATE_BEGIN_ARGS(SUndoHistory) { }
	SLATE_END_ARGS()

	virtual ~SUndoHistory();

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct( const FArguments& InArgs );
	
	//~ Begin SWidget Interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	//~ End SWidget Interface

protected:

	/**
	 * Reloads the list of undo transactions.
	 */
	void ReloadUndoList( );
	void OnUndoBufferChanged();
	void OnTransactionStateChanged(const FTransactionContext& TransactionContext, ETransactionStateEventType TransactionState);

private:

	/** Select the last transaction in the undo history. */
	void SelectLastTransaction();

	/** Callback for clicking the 'Discard History' button. */
	FReply HandleDiscardHistoryButtonClicked( );

	/** Callback for generating a row widget for the message list view. */
	TSharedRef<ITableRow> HandleUndoListGenerateRow( TSharedPtr<FTransactionInfo> TransactionInfo, const TSharedRef<STableViewBase>& OwnerTable );

	/** Callback for when a user wants to jump to a certain transaction. */ 
	void HandleGoToTransaction(const FGuid& TargetTransactionId);

	/** Callback for checking if the given index is the current undo barrier index. */
	bool HandleIsCurrentUndoBarrier(int32 InQueueIndex) const;

	/** Callback for checking if the given index is the last active transaction. */
	bool HandleIsLastTransactionIndex(int32 InQueueIndex) const;

	/** Callback for when a user clicks to add or remove an undo barrier. */
	void HandleUndoBarrierButtonClicked(int32 InQueueIndex);

	/** Callback for checking whether the specified undo list row transaction is applied. */
	bool HandleUndoListRowIsApplied( int32 QueueIndex ) const;

	/** Callback for selecting a message in the message list view. */
	void HandleUndoListSelectionChanged( TSharedPtr<FTransactionInfo> TransactionInfo, ESelectInfo::Type SelectInfo );

	/** Callback for getting the undo size text. */
	FText HandleUndoSizeTextBlockText( ) const;

	/** Callback for getting the right undo size text color. */
	EVisibility HandleUndoWarningVisibility() const;

	/** Callback for clicking on a transaction to display details. */
	void HandleUndoListJumpToTransaction(TSharedPtr<FTransactionInfo> InItem);

	/** Callback to handle UndoHistoryDetails visibility. */
	EVisibility HandleUndoHistoryDetailsVisibility() const;

	/** Callback for getting the view button's content. */
	TSharedRef<SWidget> GetViewButtonContent() const;

	/** Toggle visibility of the transaction details section. */
	void ToggleShowTransactionDetails();

	/** Whether the transaction details section should be displayed. */
	bool IsShowingTransactionDetails() const;

private:

	/** Holds the index of the last active transaction. */
	int32 LastActiveTransactionIndex;

	/** Holds the list of undo transaction indices. */
	TArray<TSharedPtr<FTransactionInfo> > UndoList;

	/** Holds the undo list view. */
	TSharedPtr<SListView<TSharedPtr<FTransactionInfo> > > UndoListView;

	/** Holds the undo details panel view. */
	TSharedPtr<class SUndoHistoryDetails> UndoDetailsView;

	/** Holds the Transaction panel splitter. */
	TSharedPtr<SSplitter> Splitter;

	/** Holds the Details slot from the splitter. */
	TSharedPtr<SSplitter::FSlot> DetailsSlot;

	/** Holds the undo history discard button. */
	TSharedPtr<SButton> DiscardButton;
};
