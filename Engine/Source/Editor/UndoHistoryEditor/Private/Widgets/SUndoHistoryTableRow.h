// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor.h"
#include "Misc/Attribute.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Editor/Transactor.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Styling/AppStyle.h"
#include "SPositiveActionButton.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SButton.h"
#include "SNegativeActionButton.h"

#define LOCTEXT_NAMESPACE "SUndoHistoryTableRow"

DECLARE_DELEGATE_OneParam(FOnGotoTransactionClicked, const FGuid&)
DECLARE_DELEGATE_OneParam(FOnUndoBarrierButtonClicked, int32)

/**
 * Implements a row widget for the undo history list.
 */
class SUndoHistoryTableRow
	: public SMultiColumnTableRow<TSharedPtr<int32> >
{

public:

	SLATE_BEGIN_ARGS(SUndoHistoryTableRow) { }
		SLATE_ATTRIBUTE(bool, IsApplied)
		SLATE_ATTRIBUTE(bool, IsCurrentUndoBarrier)
		SLATE_ATTRIBUTE(bool, IsLastTransactionIndex)
		SLATE_ARGUMENT(int32, QueueIndex)
		SLATE_ARGUMENT(const FTransaction*, Transaction)
		SLATE_EVENT(FOnGotoTransactionClicked, OnGotoTransactionClicked)
		SLATE_EVENT(FOnUndoBarrierButtonClicked, OnUndoBarrierButtonClicked)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		IsApplied = InArgs._IsApplied;
		IsCurrentUndoBarrier = InArgs._IsCurrentUndoBarrier;
		IsLastTransactionIndex = InArgs._IsLastTransactionIndex;
		QueueIndex = InArgs._QueueIndex;
		OnGotoTransactionClicked = InArgs._OnGotoTransactionClicked;
		OnUndoBarrierButtonClicked = InArgs._OnUndoBarrierButtonClicked;

		FSuperRowType::FArguments Args = FSuperRowType::FArguments()
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));

		UObject* ContextObject = nullptr;
		if (InArgs._Transaction)
		{
			TransactionId = InArgs._Transaction->GetId();
			ContextObject = InArgs._Transaction->GetContext().PrimaryObject;
			Title = InArgs._Transaction->GetTitle();
			Description = InArgs._Transaction->GetDescription();
		}

		if (ContextObject != nullptr)
		{
			Title = FText::Format(LOCTEXT("UndoHistoryTableRowTitleF", "{0} [{1}]"), InArgs._Transaction->GetTitle(), FText::FromString(ContextObject->GetFName().ToString()));
		}

		SMultiColumnTableRow<TSharedPtr<int32> >::Construct(Args, InOwnerTableView);
	}

public:

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == "JumpToButton")
		{
			return SNew(SPositiveActionButton)
				.ToolTipText(LOCTEXT("JumpToTransactionTooltip", "Jump to this transaction"))
				.Icon(FAppStyle::GetBrush("Icons.CircleArrowRight"))
				.OnClicked_Lambda([this]() { OnGotoTransactionClicked.ExecuteIfBound(TransactionId); return FReply::Handled(); })
				.Visibility_Lambda([this]() { return this->IsHovered() ? EVisibility::Visible : EVisibility::Hidden; });
		}
		else if (ColumnName == "Title")
		{
			return SNew(SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock)
						.Text(Title)
						.ToolTipText(Description)
						.ColorAndOpacity(this, &SUndoHistoryTableRow::HandleTitleTextColorAndOpacity)
				];
		}
		else if (ColumnName == "UndoBarrierButton")
		{
			return SNew(SNegativeActionButton)
				.ToolTipText_Lambda( [this]() { if (IsCurrentUndoBarrier.Get()) { return LOCTEXT("RemoveUndoBarrierTooltip", "Remove Undo Barrier"); } return LOCTEXT("SetUndoBarrierTooltip", "Set Undo Barrier"); })
				.Icon(FAppStyle::GetBrush("Icons.PlusCircle"))
				.OnClicked_Lambda([this]() { OnUndoBarrierButtonClicked.ExecuteIfBound(QueueIndex); return FReply::Handled(); })
				.Visibility_Lambda([this]() { return (GEditor && (!GEditor->PlayWorld || GEditor->bIsSimulatingInEditor)) && this->IsHovered() && (IsCurrentUndoBarrier.Get() || IsLastTransactionIndex.Get()) ? EVisibility::Visible : EVisibility::Hidden; });
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	/** Callback for getting the color of the 'Title' text. */
	FSlateColor HandleTitleTextColorAndOpacity( ) const
	{
		if (IsApplied.Get())
		{
			return FSlateColor::UseForeground();
		}

		return FSlateColor::UseSubduedForeground();
	}

private:

	/** Holds an attribute that determines whether the transaction in this row is applied. */
	TAttribute<bool> IsApplied;

	/** Holds an attribute that determines whether the transaction in this row is the current undo barrier. */
	TAttribute<bool> IsCurrentUndoBarrier;

	/** Holds an attribute that determines whether the transaction in this row is the last active transaction. */
	TAttribute<bool> IsLastTransactionIndex;

	/** Holds the transaction's index in the transaction queue. */
	int32 QueueIndex;

	/** Holds the current transaction's id. */
	FGuid TransactionId;

	/** Holds the transaction's title text. */
	FText Title;

	/** Holds the transaction's description text. */
	FText Description;

	/** Delegate called when the Goto button is clicked */
	FOnGotoTransactionClicked OnGotoTransactionClicked;

	/** Delegate called when the Barrier button is clicked */
	FOnUndoBarrierButtonClicked OnUndoBarrierButtonClicked;
};


#undef LOCTEXT_NAMESPACE
