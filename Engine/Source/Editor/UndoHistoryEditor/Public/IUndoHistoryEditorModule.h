// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

UNDOHISTORYEDITOR_API extern const FName UndoHistoryTabName;

class IUndoHistoryEditorModule : public IModuleInterface
{
public:

	static IUndoHistoryEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IUndoHistoryEditorModule>("UndoHistoryEditor");
	}

	virtual void ExecuteOpenUndoHistory() = 0;
};
