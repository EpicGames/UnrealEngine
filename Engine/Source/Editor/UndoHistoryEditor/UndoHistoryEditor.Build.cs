// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class UndoHistoryEditor : ModuleRules
	{
		public UndoHistoryEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			// Enable truncation warnings in this module
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"InputCore",
					"Slate",
					"SlateCore",
					"ToolWidgets",
					"UndoHistory",
					"UnrealEd",
				}
			);
		}
	}
}
